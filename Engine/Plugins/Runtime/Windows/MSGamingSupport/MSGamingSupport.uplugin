{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Configuration for Microsoft GDK Plug-ins for Unreal Engine",
	"Description": "Microsoft Gaming Runtime and Microsoft Game Store plugins.",
	"CreatedBy": "Epic Games, Inc.",
	"Category": "Platform",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"RequiresBuildPlatform": false,
	"CanContainContent": false,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "MSGamingSupportEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64" ]
		}
	]
}