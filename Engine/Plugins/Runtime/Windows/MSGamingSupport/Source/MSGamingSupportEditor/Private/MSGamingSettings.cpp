// Copyright Epic Games, Inc. All Rights Reserved.

#include "MSGamingSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MSGamingSettings)

UMSGamingSettings::UMSGamingSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

