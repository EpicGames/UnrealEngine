// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GDKTargetSettings.h"
#include "MSGamingSettings.generated.h"

/**
 * Implements the settings for the MS Gaming plugins
 */
UCLASS(MinimalAPI, config = Engine, defaultconfig, ConfigDoNotCheckDefaults)
class UMSGamingSettings
	: public UGDKTargetSettings
{
public:

	GENERATED_UCLASS_BODY()

	virtual const TCHAR* GetPlatformName() const override
	{
		return TEXT("Win64");
	}

	void virtual OverrideConfigSection(FString& InOutSectionName) override
	{
		InOutSectionName = TEXT("/Script/MSGamingSupport.MSGamingSettings");
	}


	/* Automatically generate an MSIXVC package during Win64 packaging */
	UPROPERTY(EditAnywhere, Config, Category=Packaging)
	bool bAutoGeneratePackage = true;

	/* Whether to always include DX11 dependency, regardless of the 'default rhi' setting. Use this if you plan to expose bUseD3D12InGame to users, or provide an alternative method of choosing an RHI */
	UPROPERTY(EditAnywhere, Config, Category = Packaging, Meta=(DisplayName="Always Include DX11 Dependency"))
	bool bAlwaysIncludeDX11Dependency = false;

	/* Whether to automatically include GameInputRedist in the custom install actions */
	UPROPERTY(EditAnywhere, Config, Category = Packaging, Meta=(DisplayName="Always Include GameInput Dependency"))
	bool bAlwaysIncludeGameInputDependency = false;

	/* Custom installation actions. See the GDK documentation for details */
	UPROPERTY(EditAnywhere, Config, Category=Packaging, Meta=(TitleProperty=Name))
	TArray<FGDKCustomInstallAction> CustomInstallActions;

	/* Relative path in the package where the custom install actions are staged. Defaults to a folder under the root of the package */
	UPROPERTY(EditAnywhere, Config, Category=Packaging)
	FString CustomInstallStagingRoot;

	/* Blocks the game from installing or launching on a Windows version older than this. Set to a four-part version number (e.g., 10.0.18362.0) or leave blank to use the GDK's default value. */ 
	UPROPERTY(EditAnywhere, Config, Category=Packaging, AdvancedDisplay)
	FString RequiredMinimumWindowsVersion;

	/* The Windows version displayed as the Minimum Requirements on Microsoft store fronts. Set to a four-part version number (e.g., 10.0.18362.0) or leave blank to use the GDK's default value. */
	UPROPERTY(EditAnywhere, Config, Category=Packaging, AdvancedDisplay)
	FString SuggestedMinimumWindowsVersion;

	/* The Windows version displayed as the Recommended Requirements on Microsoft store fronts. Set to a four-part version number (e.g., 10.0.18362.0) or leave blank to use the GDK's default value. */
	UPROPERTY(EditAnywhere, Config, Category=Packaging, AdvancedDisplay)
	FString RecommendedWindowsVersion;

	/* Whether to restart the GDK runtime between Play-In-Editor runs */
	UPROPERTY(EditAnywhere, Config, Category=Features)
	bool bRestartRuntimeForPIE = true;

	/* Whether the GDK runtime will be initialized on startup or the first time is is needed. Note: Enabling the GDK OSS plugin will override this and force initialization on startup */
	UPROPERTY(EditAnywhere, Config, Category=Features, Meta=(DisplayName="Lazy Runtime initialize"))
	bool bLazyInitialize = true;
};
