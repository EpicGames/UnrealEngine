// Copyright Epic Games, Inc. All Rights Reserved.

#include "MSGamingSettingsDetails.h"

#include "MSGamingSettings.h"
#include "Widgets/Input/SButton.h"
#include "ObjectEditorUtils.h"
#include "IPropertyUtilities.h"
#include "DetailLayoutBuilder.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Interfaces/IPluginManager.h"
#include "SourceControlOperations.h"


#define LOCTEXT_NAMESPACE "MSGamingSettingsDetails"

FMSGamingSettingsDetails::FMSGamingSettingsDetails()
	: FGDKTargetSettingsDetails(TEXT("Win64"))
{
}

void FMSGamingSettingsDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FGDKTargetSettingsDetails::CustomizeDetails(DetailBuilder);

#if 0 // disabling this for the moment because a developer may have the MSGameStore plugin enabled in a custom build target but would still need to edit the packaging properties in the editor. Can revisit in the future - perhaps grouping these items under a separarate 'packaging' banner?
	// hide packaging-related properties unless the MSGameStore plugin is enabled
	TSharedPtr<IPlugin> MSGameStorePlugin = IPluginManager::Get().FindPlugin(TEXT("MSGameStore"));
	if (!MSGameStorePlugin.IsValid() || !MSGameStorePlugin->IsEnabled())
	{
		const char* PackagingCategories[] =
		{
			"Chunk Install",
			"Packaging",
		};

		for (const char* PackagingCategory : PackagingCategories)
		{
			DetailBuilder.HideCategory(PackagingCategory);
		}

		TSharedRef<IPropertyHandle> TopLevelProperty = DetailBuilder.GetProperty( DetailBuilder.GetTopLevelProperty() );
		HideMSGameStoreProperties(DetailBuilder, TopLevelProperty);
	}
#endif
}

void FMSGamingSettingsDetails::HideMSGameStoreProperties(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> PropertyHandle)
{
	static const FName ForMSGameStoreMetaData("ForMSGameStore");
	if (PropertyHandle->HasMetaData(ForMSGameStoreMetaData))
	{
		DetailBuilder.HideProperty(PropertyHandle);
	}

	uint32 NumChildren = 0;
	PropertyHandle->GetNumChildren(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
	{
		TSharedRef<IPropertyHandle> ChildPropertyHandle = PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		HideMSGameStoreProperties(DetailBuilder, ChildPropertyHandle);
	}
}

void FMSGamingSettingsDetails::AddAdditionalGettingStartedItems( TSharedPtr<class SHorizontalBox> GettingStartedBox, TWeakObjectPtr<class UGDKTargetSettings> GDKTargetSettings, const TSharedRef<class IPropertyUtilities> PropertyUtilities )
{
	UClass* WinGDKTargetSettingsClass = FindObject<UClass>(nullptr, TEXT("/Script/WinGDKPlatformEditor.WinGDKTargetSettings") ); //NB. this is the actual object name, not the ini config section which is overridden via UWinGDKTargetSettings::OverrideConfigSection
	TWeakObjectPtr<UGDKTargetSettings> WinGDKTargetSettings = WinGDKTargetSettingsClass ? Cast<UGDKTargetSettings>(WinGDKTargetSettingsClass->GetDefaultObject()) : nullptr;

	if (WinGDKTargetSettings != nullptr && IsGDKConfigured(WinGDKTargetSettings.Get()))
	{
		GettingStartedBox->AddSlot()
		.AutoWidth()
		.Padding(8,0,0,0)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.Text(LOCTEXT("ImportFromWinGDK", "Import From WinGDK Platform"))
			.ToolTipText(LOCTEXT("ImportFromWinGDKToolTip", "Copy the settings & packaging resources from the WinGDK platform to get things started"))
			.ButtonStyle(FAppStyle::Get(), "Button")
			.OnClicked_Lambda([this, GDKTargetSettings, WinGDKTargetSettings, PropertyUtilities]()
			{
				OnClickImportFromWinGDK(GDKTargetSettings.Get(), WinGDKTargetSettings.Get(), PropertyUtilities);
				return FReply::Handled();
			})
		];
	}
}

void FMSGamingSettingsDetails::OnClickImportFromWinGDK( UGDKTargetSettings* GDKTargetSettings, UGDKTargetSettings* WinGDKTargetSettings, const TSharedRef<class IPropertyUtilities> PropertyUtilities )
{
	// copy over all properties that we can
	for (FProperty* Property = WinGDKTargetSettings->GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		FName PropertyName = Property->GetFName();
		if (FProperty* DestProperty = FindFProperty<FProperty>(GDKTargetSettings->GetClass(), PropertyName))
		{
			// try to copy the property
			if (!FObjectEditorUtils::MigratePropertyValue(WinGDKTargetSettings, Property, GDKTargetSettings, DestProperty))
			{
				continue;
			}

			NotifyPropertyChanged(DestProperty, GDKTargetSettings, PropertyUtilities);
		}
	}

	// copy packaging resources
	const FString SrcDir = FPaths::ConvertRelativePathToFull( FPaths::ProjectPlatformExtensionDir(TEXT("WinGDK")) / TEXT("Build/Resources/") ).Replace(TEXT("/"), TEXT("\\"));
	const FString DstDir = FPaths::ConvertRelativePathToFull( FPaths::ProjectDir() / TEXT("Build/Windows/MSGaming/Resources/") ).Replace(TEXT("/"), TEXT("\\"));

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	PlatformFile.CreateDirectoryTree(*DstDir);

	bool bResourceFilesCopied = false;
	bool bResourceFilesAddedToSourceControl = false;
	const TCHAR* ResourceFileNames[] = 
	{ 
		TEXT("Logo.png"), 
		TEXT("StoreLogo.png"), 
		TEXT("SplashScreen.png"), 
		TEXT("SmallLogo.png"), 
		TEXT("Square480x480Logo.png") 
	};
	for (int Index = 0; Index < UE_ARRAY_COUNT(ResourceFileNames); Index++)
	{
		const FString SrcFile = SrcDir / ResourceFileNames[Index];
		const FString DstFile = DstDir / ResourceFileNames[Index];
		if (!PlatformFile.FileExists(*DstFile) && PlatformFile.CopyFile(*DstFile, *SrcFile) )
		{
			bResourceFilesCopied = true;
			if (ISourceControlModule::Get().IsEnabled())
			{
				ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
				bResourceFilesAddedToSourceControl |= (SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), DstFile) == ECommandResult::Succeeded);
			}
		}
	}


	// refresh the property editor UI
	PropertyUtilities->ForceRefresh();

	// display packaging resource copy result
	FText DstDirText = FText::FromString(*DstDir);
	if (bResourceFilesAddedToSourceControl)
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format( LOCTEXT("ResImagesInSC", "Packaging resource images have copied to \"{0}\" and added to source control"), DstDirText ) );
	}
	else if (bResourceFilesCopied)
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format( LOCTEXT("ResImagesCopied", "Packaging resource images have copied to \"{0}\""), DstDirText ) );
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format( LOCTEXT("ResImagesNotCopied", "No packaging resource images have been copied to \"{0}\""), DstDirText ) );
	}

	// show the packaging resource folder
	FPlatformProcess::ExploreFolder(*DstDir);
}

#undef LOCTEXT_NAMESPACE
