// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GDKTargetSettingsDetails.h"

/**
* Detail customization for MSGamingSettings panel
*/
class FMSGamingSettingsDetails : public FGDKTargetSettingsDetails
{
public:
	FMSGamingSettingsDetails();
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShareable(new FMSGamingSettingsDetails);
	}

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	void HideMSGameStoreProperties(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> PropertyHandle);

	void OnClickImportFromWinGDK( class UGDKTargetSettings* GDKTargetSettings, UGDKTargetSettings* WinGDKTargetSettings, const TSharedRef<class IPropertyUtilities> PropertyUtilities );
	virtual void AddAdditionalGettingStartedItems( TSharedPtr<class SHorizontalBox> GettingStartedBox, TWeakObjectPtr<class UGDKTargetSettings> GDKTargetSettings, const TSharedRef<class IPropertyUtilities> PropertyUtilities ) override;

};
