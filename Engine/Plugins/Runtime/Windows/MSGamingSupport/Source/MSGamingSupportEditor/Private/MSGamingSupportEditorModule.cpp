// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "ISettingsModule.h"
#include "MSGamingSettings.h"
#include "MSGamingSettingsDetails.h"

#define LOCTEXT_NAMESPACE "MSGamingSupportEditorModule"

class FMSGamingSupportEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		// register detail customization
		FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
		if (PropertyModule != nullptr)
		{
			static FName MSGamingSettings("MSGamingSettings");
			PropertyModule->RegisterCustomClassLayout(MSGamingSettings, FOnGetDetailCustomizationInstance::CreateStatic(&FMSGamingSettingsDetails::MakeInstance));
		}

		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "MS Gaming",
				LOCTEXT("MSGSettingName", "MS Gaming (PC GDK)"),
				LOCTEXT("MSGSettingsDescription", "Configure the MS Gaming plugins"),
				GetMutableDefault<UMSGamingSettings>()
			);
		}
	}

	virtual void ShutdownModule() override
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "MS Gaming");
		}
	}
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMSGamingSupportEditorModule, MSGamingSupportEditor);
